{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Property Access Editor",
	"Description" : "Editor support for copying properties from one object to another. Required for Animation and UMG systems to function correctly",
	"Category" : "Runtime",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	
	"Modules" :
	[
		{
			"Name" : "PropertyAccessEditor",
			"Type": "UncookedOnly" ,
			"LoadingPhase" : "PreDefault"
		}		
	],
	"SupportedPrograms": ["ChaosVisualDebugger", "LiveLinkHub"]
}
