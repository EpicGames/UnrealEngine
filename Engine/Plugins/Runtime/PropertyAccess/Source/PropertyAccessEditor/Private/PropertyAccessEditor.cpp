// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAccessEditor.h"
#include "PropertyAccess.h"
#include "PropertyPathHelpers.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintActionDatabase.h"
#include "Settings/AnimBlueprintSettings.h"
#include "Preferences/PersonaOptions.h"

#define LOCTEXT_NAMESPACE "PropertyAccessEditor"

struct FPropertyAccessEditorSystem
{
	struct FResolveSegmentsContext
	{
		FResolveSegmentsContext(const UStruct* InStruct, TArrayView<const FString> InPath, FPropertyAccessPath& InAccessPath)
			: Struct(InStruct)
			, CurrentStruct(InStruct)
			, Path(InPath)
			, AccessPath(InAccessPath)
		{}

		// Starting struct
		const UStruct* Struct = nullptr;

		// Current struct
		const UStruct* CurrentStruct = nullptr;

		// Path as FStrings with optional array markup
		TArrayView<const FString> Path;

		// The access path we are building
		FPropertyAccessPath& AccessPath;

		// Output segments
		TArray<FPropertyAccessSegment> Segments;

		// The last error message produced
		FText ErrorMessage;

		// The current segment index (or that at which the last error occurred)
		int32 SegmentIndex = INDEX_NONE;

		// Whether this is the final segment
		bool bFinalSegment = false;

		// Whether this path was determined to be thread safe
		bool bWasThreadSafe = true;

		// Whether to apply allow-list to functions and properties along the path
		bool bPerformValidation = false;

		// Whether to use the most up to date classes when traversing the path. 
		// This can be useful for situations where we are resolving against potentially out of date
		// classes, but the resulting path will not be valid to use or persist due to functions and properties
		// being on skeleton classes 
		bool bUseMostUpToDateClasses = false;
	};

	// The result of a segment resolve operation
	enum class ESegmentResolveResult
	{
		Failed,

		Succeeded,
	};

	static ESegmentResolveResult ResolveSegments_CheckProperty(FPropertyAccessSegment& InSegment, FProperty* InProperty, FResolveSegmentsContext& InContext, bool& bOutThreadSafe)
	{
		InSegment.Property = InProperty;

		// Validate the property is accessible according to permissions. Allow return properties as the owning function will have been validated.
		if (InContext.bPerformValidation && InProperty && !InProperty->HasAnyPropertyFlags(CPF_ReturnParm) && !GetDefault<UPersonaOptions>()->IsAllowedProperty(InProperty))
		{
            InContext.ErrorMessage = FText::Format(LOCTEXT("PropertyIsDisallowedByPermissions", "Property '{0}' is not allowed for property access for @@"), FText::FromName(InProperty->GetFName()));
            return ESegmentResolveResult::Failed;
		}
		
		// Check to see if it is an array first, as arrays get handled the same for 'leaf' and 'branch' nodes
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty);
		if(ArrayProperty != nullptr && InSegment.ArrayIndex != INDEX_NONE)
		{
			// It is an array, now check to see if this is an array of structures
			if(FStructProperty* ArrayOfStructsProperty = CastField<FStructProperty>(ArrayProperty->Inner))
			{
				InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::ArrayOfStructs;
				InSegment.Struct = ArrayOfStructsProperty->Struct;
			}
			// if it's not an array of structs, maybe it's an array of objects
			else if(FObjectPropertyBase* ArrayOfObjectsProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner))
			{
				InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::ArrayOfObjects;
				InSegment.Struct = ArrayOfObjectsProperty->PropertyClass;
				if(!InContext.bFinalSegment)
				{
					bOutThreadSafe = false;
				}
			}
			else
			{
				InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::Array;
			}
		}
		// Leaf segments all get treated the same, plain, struct or object
		else if(InContext.bFinalSegment)
		{
			InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::Leaf;
			InSegment.Struct = nullptr;
		}
		// Check to see if this is a simple structure (eg. not an array of structures)
		else if(FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::Struct;
			InSegment.Struct = StructProperty->Struct;
		}
		// Check to see if this is a simple object (eg. not an array of objects)
		else if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
		{
			InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::Object;
			InSegment.Struct = ObjectProperty->PropertyClass;
			if(!InContext.bFinalSegment)
			{
				bOutThreadSafe = false;
			}
		}
		// Check to see if this is a simple weak object property (eg. not an array of weak objects).
		else if(FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(InProperty))
		{
			InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::WeakObject;
			InSegment.Struct = WeakObjectProperty->PropertyClass;
			if(!InContext.bFinalSegment)
			{
				bOutThreadSafe = false;
			}
		}
		// Check to see if this is a simple soft object property (eg. not an array of soft objects).
		else if(FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InProperty))
		{
			InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::SoftObject;
			InSegment.Struct = SoftObjectProperty->PropertyClass;
			if(!InContext.bFinalSegment)
			{
				bOutThreadSafe = false;
			}
		}
		else
		{
			InContext.ErrorMessage = FText::Format(LOCTEXT("UnrecognisedProperty", "Property '{0}' is unrecognised in property path for @@"), InProperty ? FText::FromName(InProperty->GetFName()) : LOCTEXT("Null", "Null"));
			return ESegmentResolveResult::Failed;
		}

		return ESegmentResolveResult::Succeeded;
	}

	static ESegmentResolveResult ResolveSegments_CheckFunction(FPropertyAccessSegment& InSegment, UFunction* InFunction, FResolveSegmentsContext& InContext)
	{
		check(InFunction);

		InSegment.Function = InFunction;
		InSegment.Flags |= (uint16)EPropertyAccessSegmentFlags::Function;

		if (InContext.bPerformValidation && !FBlueprintActionDatabase::IsFunctionAllowed(InFunction, FBlueprintActionDatabase::EPermissionsContext::Property))
		{
			InContext.ErrorMessage = FText::Format(LOCTEXT("FunctionIsDisallowedByPermissions", "Function '{0}' is not allowed for property access for @@"), FText::FromName(InFunction->GetFName()));
			return ESegmentResolveResult::Failed;
		}
		
		// Functions are always 'getters', so we need to verify their form
		if(InFunction->NumParms != 1)
		{
			InContext.ErrorMessage = FText::Format(LOCTEXT("FunctionHasTooManyParameters", "Function '{0}' has too many parameters in property path for @@"), FText::FromName(InFunction->GetFName()));
			return ESegmentResolveResult::Failed;
		}

		FProperty* ReturnProperty = InFunction->GetReturnProperty();
		if(ReturnProperty == nullptr)
		{
			InContext.ErrorMessage = FText::Format(LOCTEXT("FunctionHasNoReturnValue", "Function '{0}' has no return value in property path for @@"), FText::FromName(InFunction->GetFName()));
			return ESegmentResolveResult::Failed;
		}

		// Treat the function's return value as the struct/class we want to use for the next segment
		bool bThreadSafeProperty = true;
		const ESegmentResolveResult Result = ResolveSegments_CheckProperty(InSegment, ReturnProperty, InContext, bThreadSafeProperty);
		if(Result != ESegmentResolveResult::Failed)
		{
			// Check a function's thread safety.
			// Note that this logic means that an external (ie. thread unsafe) object dereference returned from ResolveSegments_CheckProperty
			// can be overridden here if the function that returns the value promises that it is thread safe to access that object.
			// An example of this is would be something like accessing the main anim BP from a linked anim BP, where it is 'safe' to access
			// the other object while running animation updated on a worker thread.
			const bool bThreadSafeFunction = FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(InFunction); 
			InContext.bWasThreadSafe &= (bThreadSafeProperty && bThreadSafeFunction) || (!bThreadSafeProperty && bThreadSafeFunction);
		}

		return Result;
	}

	// Called at compile time to build out a segments array
	static EPropertyAccessResolveResult ResolveSegments(FResolveSegmentsContext& InContext)
	{
		check(InContext.Struct);

		if(InContext.Path.Num() > 0)
		{
			for(int32 SegmentIndex = 0; SegmentIndex < InContext.Path.Num(); ++SegmentIndex)
			{
				const FString& SegmentString = InContext.Path[SegmentIndex];

				FPropertyAccessSegment& Segment = InContext.Segments.AddDefaulted_GetRef();
				const TCHAR* PropertyNamePtr = nullptr;
				int32 PropertyNameLength = 0;
				int32 ArrayIndex = INDEX_NONE;
				PropertyPathHelpers::FindFieldNameAndArrayIndex(SegmentString.Len(), *SegmentString, PropertyNameLength, &PropertyNamePtr, ArrayIndex);
				ensure(PropertyNamePtr != nullptr);
				FString PropertyNameString = FString::ConstructFromPtrSize(PropertyNamePtr, PropertyNameLength);
				Segment.Name = FName(*PropertyNameString, FNAME_Find);
				Segment.ArrayIndex = ArrayIndex;

				InContext.SegmentIndex = SegmentIndex;
				InContext.bFinalSegment = SegmentIndex == InContext.Path.Num() - 1;

				if(InContext.CurrentStruct == nullptr)
				{
					InContext.ErrorMessage = LOCTEXT("MalformedPath", "Malformed property path for @@");
					return EPropertyAccessResolveResult::Failed;
				}

				// Obtain the property/function from the given structure definition
				FFieldVariant Field = FindUFieldOrFProperty(InContext.CurrentStruct, Segment.Name);
				if(!Field.IsValid())
				{
					InContext.ErrorMessage = FText::Format(LOCTEXT("InvalidField", "Invalid field '{0}' found in property path for @@"), FText::FromName(Segment.Name));
					return EPropertyAccessResolveResult::Failed;
				}

				if(FProperty* Property = Field.Get<FProperty>())
				{
					if(ResolveSegments_CheckProperty(Segment, Property, InContext, InContext.bWasThreadSafe) == ESegmentResolveResult::Failed)
					{
						return EPropertyAccessResolveResult::Failed;
					}
				}
				else if(UFunction* Function = Field.Get<UFunction>())
				{
					if(ResolveSegments_CheckFunction(Segment, Function, InContext) == ESegmentResolveResult::Failed)
					{
						return EPropertyAccessResolveResult::Failed;
					}
				}

				InContext.CurrentStruct = Segment.Struct;
#if WITH_EDITOR
				if (InContext.bUseMostUpToDateClasses && InContext.CurrentStruct && InContext.CurrentStruct->IsA<UClass>())
				{
					// Try to use the skeleton class if possible in editor as the regular class may have yet to be compiled
					InContext.CurrentStruct = FBlueprintEditorUtils::GetMostUpToDateClass(CastChecked<UClass>(InContext.CurrentStruct));
				}
#endif
			}

			if(InContext.Segments.Num() > 0)
			{
				return EPropertyAccessResolveResult::Succeeded;
			}
			else
			{
				InContext.ErrorMessage = LOCTEXT("NoSegments", "Unable to resolve any property path segments for @@");
				return EPropertyAccessResolveResult::Failed;
			}
		}
		else
		{
			InContext.ErrorMessage = LOCTEXT("InvalidPath", "Invalid path found for @@");
			return EPropertyAccessResolveResult::Failed;
		}
	}

	static FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, FProperty*& OutProperty, int32& OutArrayIndex)
	{
		FPropertyAccessPath AccessPath;
		FResolveSegmentsContext Context(InStruct, InPath, AccessPath);
		FPropertyAccessResolveResult Result;
		Result.Result = ResolveSegments(Context);
		Result.bIsThreadSafe = Context.bWasThreadSafe;
		if(Result.Result != EPropertyAccessResolveResult::Failed)
		{
			const FPropertyAccessSegment& LeafSegment = Context.Segments.Last();
			OutProperty = LeafSegment.Property.Get();
			OutArrayIndex = LeafSegment.ArrayIndex;
		}

		return Result;
	}

	static FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, const IPropertyAccessEditor::FResolvePropertyAccessArgs& InArgs)
	{
		FPropertyAccessPath AccessPath;
		FResolveSegmentsContext Context(InStruct, InPath, AccessPath);
		Context.bUseMostUpToDateClasses = InArgs.bUseMostUpToDateClasses;
		FPropertyAccessResolveResult Result;
		Result.Result = ResolveSegments(Context);
		Result.bIsThreadSafe = Context.bWasThreadSafe;
		if(Result.Result != EPropertyAccessResolveResult::Failed)
		{
			for(int32 SegmentIndex = 0; SegmentIndex < Context.Segments.Num(); ++SegmentIndex)
			{
				const FPropertyAccessSegment& Segment = Context.Segments[SegmentIndex];
				if(EnumHasAllFlags((EPropertyAccessSegmentFlags)Segment.Flags, EPropertyAccessSegmentFlags::Function))
				{
					if(InArgs.FunctionFunction != nullptr)
					{
						check(Segment.Function != nullptr);
						check(Segment.Property.Get());
						InArgs.FunctionFunction(SegmentIndex, Segment.Function, Segment.Property.Get());
					}
				}
				else
				{
					switch((EPropertyAccessSegmentFlags)Segment.Flags & ~EPropertyAccessSegmentFlags::ModifierFlags)
					{
					case EPropertyAccessSegmentFlags::Struct:
					case EPropertyAccessSegmentFlags::Leaf:
					case EPropertyAccessSegmentFlags::Object:
					case EPropertyAccessSegmentFlags::WeakObject:
					case EPropertyAccessSegmentFlags::SoftObject:
					{
						if(InArgs.PropertyFunction != nullptr)
						{
							check(Segment.Property.Get());
							InArgs.PropertyFunction(SegmentIndex, Segment.Property.Get(), Segment.ArrayIndex);
						}
						break;
					}
					case EPropertyAccessSegmentFlags::Array:
					case EPropertyAccessSegmentFlags::ArrayOfStructs:
					case EPropertyAccessSegmentFlags::ArrayOfObjects:
					{
						if(InArgs.ArrayFunction != nullptr)
						{
							FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Segment.Property.Get());
							InArgs.ArrayFunction(SegmentIndex, ArrayProperty, Segment.ArrayIndex);
						}
						break;
					}
					default:
						check(false);
						break;
					}
				}
			}
		}

		return Result;
	}
	
	static EPropertyAccessCopyType GetAccessType(const FPropertyAccessSegment& InSegment)
	{
		FProperty* Property = InSegment.Property.Get();
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			// use the array's inner property if we are not trying to copy the whole array
			if (InSegment.ArrayIndex != INDEX_NONE)
			{
				Property = ArrayProperty->Inner;
			}
		}

		if (CastField<FNameProperty>(Property))
		{
			return EPropertyAccessCopyType::Name;
		}
		else if (CastField<FBoolProperty>(Property))
		{
			return EPropertyAccessCopyType::Bool;
		}
		else if (CastField<FStructProperty>(Property))
		{
			return EPropertyAccessCopyType::Struct;
		}
		else if (CastField<FObjectPropertyBase>(Property))
		{
			return EPropertyAccessCopyType::Object;
		}
		else if (CastField<FArrayProperty>(Property) && Property->HasAnyPropertyFlags(CPF_EditFixedSize))
		{
			// only apply array copying rules if the destination array is fixed size, otherwise it will be 'complex'
			return EPropertyAccessCopyType::Array;
		}
		else if (Property->PropertyFlags & CPF_IsPlainOldData)
		{
			return EPropertyAccessCopyType::Plain;
		}
		else
		{
			return EPropertyAccessCopyType::Complex;
		}
	}

	static EPropertyAccessCopyType GetCopyType(const FPropertyAccessSegment& InSrcSegment, const FPropertyAccessSegment& InDestSegment, FText& OutErrorMessage)
	{
		FProperty* SrcProperty = InSrcSegment.Property.Get();
		check(SrcProperty);

		if(FArrayProperty* SrcArrayProperty = CastField<FArrayProperty>(SrcProperty))
		{
			// use the array's inner property if we are not trying to copy the whole array
			if(InSrcSegment.ArrayIndex != INDEX_NONE)
			{
				SrcProperty = SrcArrayProperty->Inner;
			}
		}

		FProperty* DestProperty = InDestSegment.Property.Get();
		check(DestProperty);

		if(FArrayProperty* DestArrayProperty = CastField<FArrayProperty>(DestProperty))
		{
			// use the array's inner property if we are not trying to copy the whole array
			if(InDestSegment.ArrayIndex != INDEX_NONE)
			{
				DestProperty = DestArrayProperty->Inner;
			}
		}

		EPropertyAccessCompatibility Compatibility = PropertyAccess::GetPropertyCompatibility(SrcProperty, DestProperty);
		if(Compatibility == EPropertyAccessCompatibility::Compatible)
		{
			return GetAccessType(InDestSegment);
		}
		else if(Compatibility == EPropertyAccessCompatibility::Promotable)
		{
			if(SrcProperty->IsA<FBoolProperty>())
			{
				if(DestProperty->IsA<FByteProperty>())
				{
					return EPropertyAccessCopyType::PromoteBoolToByte;
				}
				else if(DestProperty->IsA<FIntProperty>())
				{
					return EPropertyAccessCopyType::PromoteBoolToInt32;
				}
				else if(DestProperty->IsA<FInt64Property>())
				{
					return EPropertyAccessCopyType::PromoteBoolToInt64;
				}
				else if(DestProperty->IsA<FFloatProperty>())
				{
					return EPropertyAccessCopyType::PromoteBoolToFloat;
				}
				else if (DestProperty->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCopyType::PromoteBoolToDouble;
				}
			}
			else if(SrcProperty->IsA<FByteProperty>())
			{
				if(DestProperty->IsA<FIntProperty>())
				{
					return EPropertyAccessCopyType::PromoteByteToInt32;
				}
				else if(DestProperty->IsA<FInt64Property>())
				{
					return EPropertyAccessCopyType::PromoteByteToInt64;
				}
				else if(DestProperty->IsA<FFloatProperty>())
				{
					return EPropertyAccessCopyType::PromoteByteToFloat;
				}
				else if (DestProperty->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCopyType::PromoteByteToDouble;
				}
			}
			else if(SrcProperty->IsA<FIntProperty>())
			{
				if(DestProperty->IsA<FInt64Property>())
				{
					return EPropertyAccessCopyType::PromoteInt32ToInt64;
				}
				else if(DestProperty->IsA<FFloatProperty>())
				{
					return EPropertyAccessCopyType::PromoteInt32ToFloat;
				}
				else if (DestProperty->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCopyType::PromoteInt32ToDouble;
				}
			}
			else if (SrcProperty->IsA<FFloatProperty>())
			{
				if (DestProperty->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCopyType::PromoteFloatToDouble;
				}
			}
			else if (SrcProperty->IsA<FDoubleProperty>())
			{
				if (DestProperty->IsA<FFloatProperty>())
				{
					return EPropertyAccessCopyType::DemoteDoubleToFloat;
				}
			}
			else if (SrcProperty->IsA<FArrayProperty>() && DestProperty->IsA<FArrayProperty>())
			{
				const FArrayProperty* SrcArrayProperty = CastField<const FArrayProperty>(SrcProperty);
				const FArrayProperty* DestArrayProperty = CastField<const FArrayProperty>(DestProperty);

				if (SrcArrayProperty->Inner->IsA<FFloatProperty>())
				{
					if (DestArrayProperty->Inner->IsA<FDoubleProperty>())
					{
						return EPropertyAccessCopyType::PromoteArrayFloatToDouble;
					}
				}
				else if (SrcArrayProperty->Inner->IsA<FDoubleProperty>())
				{
					if (DestArrayProperty->Inner->IsA<FFloatProperty>())
					{
						return EPropertyAccessCopyType::DemoteArrayDoubleToFloat;
					}
				}
			}
			else if (SrcProperty->IsA<FMapProperty>() && DestProperty->IsA<FMapProperty>())
			{
				const FMapProperty* SrcMapProperty = CastField<const FMapProperty>(SrcProperty);
				const FMapProperty* DestMapProperty = CastField<const FMapProperty>(DestProperty);

				if (SrcMapProperty->ValueProp->IsA<FFloatProperty>())
				{
					if (DestMapProperty->ValueProp->IsA<FDoubleProperty>())
					{
						return EPropertyAccessCopyType::PromoteMapValueFloatToDouble;
					}
				}
				else if (SrcMapProperty->ValueProp->IsA<FDoubleProperty>())
				{
					if (DestMapProperty->ValueProp->IsA<FFloatProperty>())
					{
						return EPropertyAccessCopyType::DemoteMapValueDoubleToFloat;
					}
				}
			}
		}

		OutErrorMessage = FText::Format(LOCTEXT("CopyTypeInvalidFormat", "@@ Cannot copy property ({0} -> {1})"), FText::FromString(SrcProperty->GetCPPType()), FText::FromString(DestProperty->GetCPPType()));

		return EPropertyAccessCopyType::None;
	}

	static bool CompileCopy(const UStruct* InStruct, const FOnPropertyAccessDetermineBatchId& InOnDetermineBatchId, FPropertyAccessLibrary& InLibrary, FPropertyAccessLibraryCompiler::FQueuedCopy& OutCopy)
	{
		FPropertyAccessPath SrcAccessPath;
		FPropertyAccessPath DestAccessPath;

		FResolveSegmentsContext SrcContext(InStruct, OutCopy.SourcePath, SrcAccessPath);
		SrcContext.bPerformValidation = GetDefault<UAnimBlueprintSettings>()->bPerformValidation;

		FResolveSegmentsContext DestContext(InStruct, OutCopy.DestPath, DestAccessPath);
		DestContext.bPerformValidation = false;

		OutCopy.SourceResult = ResolveSegments(SrcContext);
		OutCopy.SourceErrorText = SrcContext.ErrorMessage;
		OutCopy.DestResult = ResolveSegments(DestContext);
		OutCopy.DestErrorText = DestContext.ErrorMessage;

		if(OutCopy.SourceResult != EPropertyAccessResolveResult::Failed && OutCopy.DestResult != EPropertyAccessResolveResult::Failed)
		{
			FText CopyTypeError;
			EPropertyAccessCopyType CopyType = GetCopyType(SrcContext.Segments.Last(), DestContext.Segments.Last(), CopyTypeError);
			if(CopyType != EPropertyAccessCopyType::None)
			{
				FPropertyAccessCopyContext CopyContext;
				CopyContext.Object = OutCopy.AssociatedObject;
				CopyContext.ContextId = OutCopy.ContextId;
				CopyContext.SourcePathAsText = PropertyAccess::MakeTextPath(OutCopy.SourcePath, InStruct);
				CopyContext.DestPathAsText = PropertyAccess::MakeTextPath(OutCopy.DestPath, InStruct);
				CopyContext.bSourceThreadSafe = DestContext.bWasThreadSafe;
				CopyContext.bDestThreadSafe = SrcContext.bWasThreadSafe;
				
				OutCopy.BatchId = InOnDetermineBatchId.IsBound() ? InOnDetermineBatchId.Execute(CopyContext) : 0;
				check(OutCopy.BatchId >= 0);
				InLibrary.CopyBatchArray.SetNum(FMath::Max(OutCopy.BatchId + 1, InLibrary.CopyBatchArray.Num()));
				
				OutCopy.BatchIndex = InLibrary.CopyBatchArray[OutCopy.BatchId].Copies.Num();
				FPropertyAccessCopy& Copy = InLibrary.CopyBatchArray[OutCopy.BatchId].Copies.AddDefaulted_GetRef();
				Copy.AccessIndex = InLibrary.SrcPaths.Num();
				Copy.DestAccessStartIndex = InLibrary.DestPaths.Num();
				Copy.DestAccessEndIndex = InLibrary.DestPaths.Num() + 1;
				Copy.Type = CopyType;

				SrcAccessPath.PathSegmentStartIndex = InLibrary.PathSegments.Num();
				SrcAccessPath.PathSegmentCount = SrcContext.Segments.Num();
				InLibrary.SrcPaths.Add(SrcAccessPath);
				InLibrary.PathSegments.Append(SrcContext.Segments);

				DestAccessPath.PathSegmentStartIndex = InLibrary.PathSegments.Num();
				DestAccessPath.PathSegmentCount = DestContext.Segments.Num();
				InLibrary.DestPaths.Add(DestAccessPath);
				InLibrary.PathSegments.Append(DestContext.Segments);

				return true;
			}
			else
			{
				OutCopy.SourceResult = EPropertyAccessResolveResult::Failed;
				OutCopy.SourceErrorText = CopyTypeError;
			}
		}

		return false;
	}

	static bool CompileAccess(const UStruct* InStruct, FPropertyAccessLibrary& InLibrary, FPropertyAccessLibraryCompiler::FQueuedAccess& OutAccess)
	{
		FPropertyAccessPath AccessPath;

		FResolveSegmentsContext Context(InStruct, OutAccess.Path, AccessPath);
		Context.bPerformValidation = false;

		OutAccess.Result = ResolveSegments(Context);
		OutAccess.ErrorText = Context.ErrorMessage;

		if (OutAccess.Result != EPropertyAccessResolveResult::Failed)
		{
			OutAccess.AccessIndex = InLibrary.DestPaths.Num();
			OutAccess.AccessType = GetAccessType(Context.Segments.Last());

			AccessPath.PathSegmentStartIndex = InLibrary.PathSegments.Num();
			AccessPath.PathSegmentCount = Context.Segments.Num();
			InLibrary.DestPaths.Add(AccessPath);
			InLibrary.PathSegments.Append(Context.Segments);

			return true;
		}

		return false;
	}
};

namespace PropertyAccess
{
	FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, FProperty*& OutProperty, int32& OutArrayIndex)
	{
		return ::FPropertyAccessEditorSystem::ResolvePropertyAccess(InStruct, InPath, OutProperty, OutArrayIndex);
	}

	FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, const IPropertyAccessEditor::FResolvePropertyAccessArgs& InArgs)
	{
		return ::FPropertyAccessEditorSystem::ResolvePropertyAccess(InStruct, InPath, InArgs);
	}

	EPropertyAccessCompatibility GetPropertyCompatibility(const FProperty* InPropertyA, const FProperty* InPropertyB)
	{
		if(InPropertyA == InPropertyB)
		{
			return EPropertyAccessCompatibility::Compatible;
		}

		if(InPropertyA == nullptr || InPropertyB == nullptr)
		{
			return EPropertyAccessCompatibility::Incompatible;
		}

		// Special case for object properties
		if(InPropertyA->IsA<FObjectPropertyBase>() && InPropertyB->IsA<FObjectPropertyBase>())
		{
			const FObjectPropertyBase* ObjectPropertyA = CastField<FObjectPropertyBase>(InPropertyA);
			const FObjectPropertyBase* ObjectPropertyB = CastField<FObjectPropertyBase>(InPropertyB);
			if(ObjectPropertyA->PropertyClass->IsChildOf(ObjectPropertyB->PropertyClass))
			{
				return EPropertyAccessCompatibility::Compatible;
			}
		}

		// Extract underlying types for enums
		if(const FEnumProperty* EnumPropertyA = CastField<const FEnumProperty>(InPropertyA))
		{
			InPropertyA = EnumPropertyA->GetUnderlyingProperty();
		}

		if(const FEnumProperty* EnumPropertyB = CastField<const FEnumProperty>(InPropertyB))
		{
			InPropertyB = EnumPropertyB->GetUnderlyingProperty();
		}

		if(InPropertyA->SameType(InPropertyB))
		{
			return EPropertyAccessCompatibility::Compatible;
		}
		else
		{
			// Not directly compatible, check for promotions
			if(InPropertyA->IsA<FBoolProperty>())
			{
				if(InPropertyB->IsA<FByteProperty>() || InPropertyB->IsA<FIntProperty>() || InPropertyB->IsA<FInt64Property>() || InPropertyB->IsA<FFloatProperty>() || InPropertyB->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
			else if(InPropertyA->IsA<FByteProperty>())
			{
				if(InPropertyB->IsA<FIntProperty>() || InPropertyB->IsA<FInt64Property>() || InPropertyB->IsA<FFloatProperty>() || InPropertyB->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
			else if(InPropertyA->IsA<FIntProperty>())
			{
				if(InPropertyB->IsA<FInt64Property>() || InPropertyB->IsA<FFloatProperty>() || InPropertyB->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
			else if (InPropertyA->IsA<FFloatProperty>())
			{
				if (InPropertyB->IsA<FDoubleProperty>())
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
			else if (InPropertyA->IsA<FDoubleProperty>())
			{
				if (InPropertyB->IsA<FFloatProperty>())
				{
					return EPropertyAccessCompatibility::Promotable;	// LWC_TODO: Incorrect! Do not ship this!
				}
			}
			else if (InPropertyA->IsA<FArrayProperty>() && InPropertyB->IsA<FArrayProperty>())
			{
				const FArrayProperty* ArrayPropertyA = CastField<const FArrayProperty>(InPropertyA);
				const FArrayProperty* ArrayPropertyB = CastField<const FArrayProperty>(InPropertyB);

				if (ArrayPropertyA->Inner->IsA<FFloatProperty>())
				{
					if (ArrayPropertyB->Inner->IsA<FDoubleProperty>())
					{
						return EPropertyAccessCompatibility::Promotable;
					}
				}
				else if (ArrayPropertyA->Inner->IsA<FDoubleProperty>())
				{
					if (ArrayPropertyB->Inner->IsA<FFloatProperty>())
					{
						return EPropertyAccessCompatibility::Promotable;
					}
				}
			}
			else if (InPropertyA->IsA<FSetProperty>() && InPropertyB->IsA<FSetProperty>())
			{
				const FSetProperty* SetPropertyA = CastField<const FSetProperty>(InPropertyA);
				const FSetProperty* SetPropertyB = CastField<const FSetProperty>(InPropertyB);

				if (SetPropertyA->ElementProp->IsA<FFloatProperty>())
				{
					if (SetPropertyB->ElementProp->IsA<FDoubleProperty>())
					{
						return EPropertyAccessCompatibility::Promotable;
					}
				}
				else if (SetPropertyA->ElementProp->IsA<FDoubleProperty>())
				{
					if (SetPropertyB->ElementProp->IsA<FFloatProperty>())
					{
						return EPropertyAccessCompatibility::Promotable;
					}
				}
			}
			else if (InPropertyA->IsA<FMapProperty>() && InPropertyB->IsA<FMapProperty>())
			{
				// We only support promoting value properties of maps. Is there a case where float/double keys are useful?
				const FMapProperty* MapPropertyA = CastField<const FMapProperty>(InPropertyA);
				const FMapProperty* MapPropertyB = CastField<const FMapProperty>(InPropertyB);

				if (MapPropertyA->ValueProp->IsA<FFloatProperty>())
				{
					if (MapPropertyB->ValueProp->IsA<FDoubleProperty>())
					{
						return EPropertyAccessCompatibility::Promotable;
					}
				}
				else if (MapPropertyA->ValueProp->IsA<FDoubleProperty>())
				{
					if (MapPropertyB->ValueProp->IsA<FFloatProperty>())
					{
						return EPropertyAccessCompatibility::Promotable;
					}
				}
			}
		}

		return EPropertyAccessCompatibility::Incompatible;
	}

	EPropertyAccessCompatibility GetPinTypeCompatibility(const FEdGraphPinType& InPinTypeA, const FEdGraphPinType& InPinTypeB)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		if (Schema->ArePinTypesCompatible(InPinTypeA, InPinTypeB))
		{
			return EPropertyAccessCompatibility::Compatible;
		}
		else
		{
			// Not directly compatible, check for promotions
			if (InPinTypeA.PinCategory == UEdGraphSchema_K2::PC_Boolean)
			{
				if (InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Byte || InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Int || InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Int64 || InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Real)
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
			else if (InPinTypeA.PinCategory == UEdGraphSchema_K2::PC_Byte)
			{
				if (InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Int || InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Int64 || InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Real)
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
			else if (InPinTypeA.PinCategory == UEdGraphSchema_K2::PC_Int)
			{
				if (InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Int64 || InPinTypeB.PinCategory == UEdGraphSchema_K2::PC_Real)
				{
					return EPropertyAccessCompatibility::Promotable;
				}
			}
		}

		return EPropertyAccessCompatibility::Incompatible;
	}

	void MakeStringPath(const TArray<FBindingChainElement>& InBindingChain, TArray<FString>& OutStringPath)
	{
		Algo::Transform(InBindingChain, OutStringPath, [](const FBindingChainElement& InElement)
		{
			if(FProperty* Property = InElement.Field.Get<FProperty>())
			{
				if(InElement.ArrayIndex == INDEX_NONE)
				{
					return Property->GetName();
				}
				else
				{
					return FString::Printf(TEXT("%s[%d]"), *Property->GetName(), InElement.ArrayIndex);
				}
			}
			else if(UFunction* Function = InElement.Field.Get<UFunction>())
			{
				return Function->GetName();
			}
			else
			{
				check(false);
				return FString();
			}
		});
	}

	FText MakeTextPath(const TArray<FString>& InPath, const UStruct* InStruct = nullptr)
	{
		TStringBuilder<128> StringBuilder;
		
		if(InPath.Num() > 0)
		{
			const int32 LastIndex = InPath.Num() - 1;
			bool bResolved = InStruct != nullptr;
			
			if(InStruct)
			{
				IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
				auto PropertyFunction = [&StringBuilder](FProperty* InProperty, int32 InArrayIndex, bool bLast)
				{
					if(InProperty->IsNative())
					{
						if(const FString* ScriptNamePtr = InProperty->FindMetaData("ScriptName"))
						{
							StringBuilder.Append(*ScriptNamePtr);
						}
						else
						{
							StringBuilder.Append(InProperty->GetName());
						}
					}
					else
					{
						StringBuilder.Append(InProperty->GetDisplayNameText().ToString());
					}

					if(InArrayIndex != INDEX_NONE)
					{
						StringBuilder.Appendf(TEXT("[%d]"), InArrayIndex);
					}

					if(!bLast)
					{
						StringBuilder.Append(TEXT("."));
					}
				};
				
				ResolveArgs.PropertyFunction = [&PropertyFunction, LastIndex](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
				{
					PropertyFunction(InProperty, InStaticArrayIndex, InSegmentIndex == LastIndex);
				};
				ResolveArgs.ArrayFunction = [&PropertyFunction, LastIndex](int32 InSegmentIndex, FArrayProperty* InProperty, int32 InArrayIndex)
				{
					PropertyFunction(InProperty, InArrayIndex, InSegmentIndex == LastIndex);
				};
				ResolveArgs.FunctionFunction = [&StringBuilder, LastIndex](int32 InSegmentIndex, UFunction* InFunction, FProperty* /*ReturnProperty*/)
				{
					if(const FString* ScriptNamePtr = InFunction->FindMetaData("ScriptName"))
					{
						StringBuilder.Append(*ScriptNamePtr);
					}
					else
					{
						StringBuilder.Append(InFunction->GetName());
					}

					if(InSegmentIndex != LastIndex)
					{
						StringBuilder.Append(TEXT("."));
					}
				};
				
				if(FPropertyAccessEditorSystem::ResolvePropertyAccess(InStruct, InPath, ResolveArgs).Result == EPropertyAccessResolveResult::Failed)
				{
					bResolved = false;
				}
			}

			// Fallback to string concatenation if we didnt/couldnt resolve 
			if(!bResolved)
			{
				StringBuilder.Append(InPath[0]);

				for(int32 SegmentIndex = 1; SegmentIndex < InPath.Num(); ++SegmentIndex)
				{
					StringBuilder.Append(TEXT("."));
					StringBuilder.Append(InPath[SegmentIndex]);
				}
			}
		}

		return FText::FromString(StringBuilder.ToString());
	}
}

FPropertyAccessLibraryCompiler::FPropertyAccessLibraryCompiler(FPropertyAccessLibrary* InLibrary, const UClass* InClass, const FOnPropertyAccessDetermineBatchId& InOnDetermineBatchId)
	: Library(InLibrary)
	, Class(InClass)
	, OnDetermineBatchId(InOnDetermineBatchId)
{
}

void FPropertyAccessLibraryCompiler::BeginCompilation()
{
	if(Class && Library)
	{
		*Library = FPropertyAccessLibrary();
	}
}

FPropertyAccessHandle FPropertyAccessLibraryCompiler::AddCopy(TArrayView<FString> InSourcePath, TArrayView<FString> InDestPath, const FName& InContextId, UObject* InAssociatedObject)
{
	FQueuedCopy QueuedCopy;
	QueuedCopy.SourcePath = InSourcePath;
	QueuedCopy.DestPath = InDestPath;
	QueuedCopy.ContextId = InContextId;
	QueuedCopy.AssociatedObject = InAssociatedObject;

	QueuedCopies.Add(MoveTemp(QueuedCopy));

	return FPropertyAccessHandle(QueuedCopies.Num() - 1, EPropertyAccessHandleType::Copy);
}

FPropertyAccessHandle FPropertyAccessLibraryCompiler::AddAccess(TArrayView<FString> InPath, UObject* InAssociatedObject)
{
	FQueuedAccess QueuedAccess;
	QueuedAccess.Path = InPath;
	QueuedAccess.AssociatedObject = InAssociatedObject;

	QueuedAccesses.Add(MoveTemp(QueuedAccess));

	return FPropertyAccessHandle(QueuedAccesses.Num() - 1, EPropertyAccessHandleType::Access);
}

bool FPropertyAccessLibraryCompiler::FinishCompilation()
{
	if(Class && Library)
	{
		bool bResult = true;
		for(int32 CopyIndex = 0; CopyIndex < QueuedCopies.Num(); ++CopyIndex)
		{
			FQueuedCopy& Copy = QueuedCopies[CopyIndex];
			bResult &= ::FPropertyAccessEditorSystem::CompileCopy(Class, OnDetermineBatchId, *Library, Copy);
			CopyMap.Add(FPropertyAccessHandle(CopyIndex, EPropertyAccessHandleType::Copy), FCompiledPropertyAccessHandle(Copy.BatchIndex, Copy.BatchId, EPropertyAccessHandleType::Copy));
		}

		for (int32 AccessIndex = 0; AccessIndex < QueuedAccesses.Num(); ++AccessIndex)
		{
			FQueuedAccess& Access = QueuedAccesses[AccessIndex];
			bResult &= ::FPropertyAccessEditorSystem::CompileAccess(Class, *Library, Access);
			AccessMap.Add(FPropertyAccessHandle(AccessIndex, EPropertyAccessHandleType::Access), FCompiledPropertyAccessHandle(Access.AccessIndex, INDEX_NONE, EPropertyAccessHandleType::Access));
		}

		// Always rebuild the library even if we detected a 'failure'. Otherwise we could fail to copy data for both
		// valid and invalid copies 
		PropertyAccess::PatchPropertyOffsets(*Library);

		return bResult;
	}

	return false;
}

void FPropertyAccessLibraryCompiler::IterateErrors(TFunctionRef<void(const FText&, UObject*)> InFunction) const
{
	if(Class && Library)
	{
		for(const FQueuedCopy& Copy : QueuedCopies)
		{
			if(Copy.SourceResult == EPropertyAccessResolveResult::Failed)
			{
				InFunction(Copy.SourceErrorText, Copy.AssociatedObject);
			}

			if(Copy.DestResult == EPropertyAccessResolveResult::Failed)
			{
				InFunction(Copy.DestErrorText, Copy.AssociatedObject);
			}
		}
	}
}

FCompiledPropertyAccessHandle FPropertyAccessLibraryCompiler::GetCompiledHandle(FPropertyAccessHandle InHandle) const
{
	if (InHandle.GetType() == EPropertyAccessHandleType::Copy)
	{
		if (const FCompiledPropertyAccessHandle* FoundHandle = CopyMap.Find(InHandle))
		{
			return *FoundHandle;
		}
	}
	else if (InHandle.GetType() == EPropertyAccessHandleType::Access)
	{
		if (const FCompiledPropertyAccessHandle* FoundHandle = AccessMap.Find(InHandle))
		{
			return *FoundHandle;
		}
	}

	return FCompiledPropertyAccessHandle();
}

EPropertyAccessCopyType FPropertyAccessLibraryCompiler::GetCompiledHandleAccessType(FPropertyAccessHandle InHandle) const
{
	if (InHandle.GetType() == EPropertyAccessHandleType::Access)
	{
		if (const FCompiledPropertyAccessHandle* FoundHandle = AccessMap.Find(InHandle))
		{
			return QueuedAccesses[InHandle.GetId()].AccessType;
		}
	}

	return EPropertyAccessCopyType::None;
}

#undef LOCTEXT_NAMESPACE
