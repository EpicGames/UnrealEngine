// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyAccessEditor.h"
#include "IPropertyAccessCompiler.h"
#include "PropertyAccess.h"

struct FEdGraphPinType;

namespace PropertyAccess
{
	/** Resolve a property path to a structure, returning the leaf property and array index if any. @return true if resolution succeeded */
	extern FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, FProperty*& OutProperty, int32& OutArrayIndex);

	/**
	 * Resolve a property path to a structure, calling back for each segment in path segment order if resolution succeed
	 * @return true if resolution succeeded
	 */
	extern FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, const IPropertyAccessEditor::FResolvePropertyAccessArgs& InArgs);

	// Get the compatibility of the two supplied properties. Ordering matters for promotion (A->B).
	extern EPropertyAccessCompatibility GetPropertyCompatibility(const FProperty* InPropertyA, const FProperty* InPropertyB);

	// Get the compatibility of the two supplied pins. Ordering matters for promotion (A->B).
	extern EPropertyAccessCompatibility GetPinTypeCompatibility(const FEdGraphPinType& InPinTypeA, const FEdGraphPinType& InPinTypeB);

	// Makes a string path from a binding chain
	extern void MakeStringPath(const TArray<FBindingChainElement>& InBindingChain, TArray<FString>& OutStringPath);

	// Makes a text path from an array of path segments 
	extern FText MakeTextPath(const TArray<FString>& InPath, const UStruct* InStruct);
}

// A helper structure used to compile a property access library
class FPropertyAccessLibraryCompiler : public IPropertyAccessLibraryCompiler
{
public:
	FPropertyAccessLibraryCompiler(FPropertyAccessLibrary* InLibrary, const UClass* InClass, const FOnPropertyAccessDetermineBatchId& InOnDetermineBatchId);

	// IPropertyAccessLibraryCompiler interface
	virtual void BeginCompilation() override;
	virtual FPropertyAccessHandle AddCopy(TArrayView<FString> InSourcePath, TArrayView<FString> InDestPath, const FName& InContextId, UObject* InAssociatedObject = nullptr) override;
	virtual FPropertyAccessHandle AddAccess(TArrayView<FString> InPath, UObject* InAssociatedObject = nullptr) override;
	virtual bool FinishCompilation() override;
	virtual void IterateErrors(TFunctionRef<void(const FText&, UObject*)> InFunction) const override;
	virtual FCompiledPropertyAccessHandle GetCompiledHandle(FPropertyAccessHandle InHandle) const override;
	virtual EPropertyAccessCopyType GetCompiledHandleAccessType(FPropertyAccessHandle InHandle) const override;

public:
	// Stored copy info for processing in FinishCompilation()
	struct FQueuedCopy
	{
		// The path to copy from
		TArray<FString> SourcePath;

		// The path to copy to
		TArray<FString> DestPath;

		// The user-defined context ID
		FName ContextId = NAME_None;

		// Error text associated with the source path
		FText SourceErrorText;

		// Error text associated with the dest path
		FText DestErrorText;

		// Result of the source path resolution
		EPropertyAccessResolveResult SourceResult = EPropertyAccessResolveResult::Failed;

		// Result of the dest path resolution
		EPropertyAccessResolveResult DestResult = EPropertyAccessResolveResult::Failed;

		// Associated object, used to provide context for path resolution
		UObject* AssociatedObject = nullptr;

		// Batch Id
		int32 BatchId = INDEX_NONE;
		
		// Index within the batch
		int32 BatchIndex = INDEX_NONE;
	};

	// Stored access info for processing in FinishCompilation()
	struct FQueuedAccess
	{
		// The path to access
		TArray<FString> Path;

		// Error text associated with the path
		FText ErrorText;

		// Result of the path resolution
		EPropertyAccessResolveResult Result = EPropertyAccessResolveResult::Failed;

		// Associated object, used to provide context for path resolution
		UObject* AssociatedObject = nullptr;

		// Compiled access index
		int32 AccessIndex = INDEX_NONE;

		// Compiled access type
		EPropertyAccessCopyType AccessType = EPropertyAccessCopyType::None;
	};

protected:
	friend struct FPropertyAccessEditorSystem;

	// The library we are compiling
	FPropertyAccessLibrary* Library;

	// The class we are compiling the library for
	const UClass* Class;

	// Delegate used to determine batch ID (index) for a particular copy context
	FOnPropertyAccessDetermineBatchId OnDetermineBatchId;
	
	// All copies to process in FinishCompilation()
	TArray<FQueuedCopy> QueuedCopies;

	// All accesses to process in FinishCompilation()
	TArray<FQueuedAccess> QueuedAccesses;

	// Copy map used to resolve handles
	TMap<FPropertyAccessHandle, FCompiledPropertyAccessHandle> CopyMap;

	// Access map used to resolve handles
	TMap<FPropertyAccessHandle, FCompiledPropertyAccessHandle> AccessMap;
};
