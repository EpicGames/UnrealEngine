// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "SPropertyBinding.h"
#include "PropertyAccessEditor.h"
#include "Features/IModularFeatures.h"

class FPropertyAccessEditorModule : public IPropertyAccessEditor, public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature("PropertyAccessEditor", this);
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature("PropertyAccessEditor", this);
	}

	// IPropertyAccessEditor interface
	virtual TSharedRef<SWidget> MakePropertyBindingWidget(UBlueprint* InBlueprint, const FPropertyBindingWidgetArgs& InArgs) const override
	{
		TArray<FBindingContextStruct> BindingContextStructs;
		return SNew(SPropertyBinding, InBlueprint, BindingContextStructs)
			.Args(InArgs);
	}

	
	virtual void FillPropertyMenu(FMenuBuilder& MenuBuilder, const FPropertyBindingWidgetArgs& InArgs, FName InPropertyName, UStruct* InOwnerStruct, TArray<TSharedPtr<FBindingChainElement>> InBindingChain) const override
	{
		SPropertyBinding::FillPropertyMenu(MenuBuilder, InArgs, nullptr, InPropertyName, InOwnerStruct, InBindingChain);
	}
	
	virtual TSharedRef<SWidget> MakePropertyBindingWidget(const TArray<FBindingContextStruct>& InBindingContextStructs, const FPropertyBindingWidgetArgs& InArgs) const override
	{
		return SNew(SPropertyBinding, nullptr, InBindingContextStructs)
			.Args(InArgs);
	}

	virtual FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, FProperty*& OutProperty, int32& OutArrayIndex) const override
	{
		return PropertyAccess::ResolvePropertyAccess(InStruct, InPath, OutProperty, OutArrayIndex);
	}

	virtual FPropertyAccessResolveResult ResolvePropertyAccess(const UStruct* InStruct, TArrayView<const FString> InPath, const FResolvePropertyAccessArgs& InArgs) const override
	{
		return PropertyAccess::ResolvePropertyAccess(InStruct, InPath, InArgs);
	}
	
	virtual EPropertyAccessCompatibility GetPropertyCompatibility(const FProperty* InPropertyA, const FProperty* InPropertyB) const override
	{
		return PropertyAccess::GetPropertyCompatibility(InPropertyA, InPropertyB);
	}

	virtual EPropertyAccessCompatibility GetPinTypeCompatibility(const FEdGraphPinType& InPinTypeA, const FEdGraphPinType& InPinTypeB) const override
	{
		return PropertyAccess::GetPinTypeCompatibility(InPinTypeA, InPinTypeB);
	}

	virtual void MakeStringPath(const TArray<FBindingChainElement>& InBindingChain, TArray<FString>& OutStringPath) const override
	{
		PropertyAccess::MakeStringPath(InBindingChain, OutStringPath);
	}

	virtual TUniquePtr<IPropertyAccessLibraryCompiler> MakePropertyAccessCompiler(const FPropertyAccessLibraryCompilerArgs& InArgs) const override
	{
		return MakeUnique<FPropertyAccessLibraryCompiler>(&InArgs.Library, InArgs.ClassContext, InArgs.OnDetermineBatchId);
	}

	virtual FText MakeTextPath(const TArray<FString>& InPath, const UStruct* InStruct = nullptr) const override
	{
		return PropertyAccess::MakeTextPath(InPath, InStruct);
	};
};

IMPLEMENT_MODULE(FPropertyAccessEditorModule, PropertyAccessEditor)
