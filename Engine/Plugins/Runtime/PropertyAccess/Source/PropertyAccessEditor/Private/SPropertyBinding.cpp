// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyBinding.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SNullWidget.h"

#if WITH_EDITOR
#include "Engine/BlueprintGeneratedClass.h"
#endif // WITH_EDITOR


#include "DetailLayoutBuilder.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "UObject/UObjectIterator.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BlueprintActionDatabase.h"
#include "Preferences/PersonaOptions.h"
#include "Styling/SlateTypes.h"

#define LOCTEXT_NAMESPACE "PropertyBinding"

/////////////////////////////////////////////////////
// SPropertyBinding

void SPropertyBinding::Construct(const FArguments& InArgs, UBlueprint* InBlueprint, const TArray<FBindingContextStruct>& InBindingContextStructs)
{
	Blueprint = InBlueprint;
	BindingContextStructs = InBindingContextStructs;

	Args = InArgs._Args;
	PropertyName = Args.Property != nullptr ? Args.Property->GetFName() : NAME_None;

	for(int32 Index = 0; Index < BindingContextStructs.Num(); ++Index)
	{
		FBindingContextStruct& ContextStruct = BindingContextStructs[Index];
		/* BindingContextStructSections contains an entry for each different FBindingContextStruct.Section.
		 Section are then further divided into subcategories based on the FBindingContextStruct.Category
		 Section are displayed as a section while subcategories are displayed as menu and submenu under a section. */
		FBindingContextStructCategory* Section = BindingContextStructSections.FindByPredicate([&](const FBindingContextStructCategory& Section) { return Section.Name.IdenticalTo(ContextStruct.Section); });
		if(!Section)
		{
			Section = &BindingContextStructSections.AddDefaulted_GetRef();
			Section->Name = ContextStruct.Section;
		}

		//No category the BindingContextStruct will be displayed directly inside the section.
		if (ContextStruct.Category.IsEmpty())
		{
			Section->BindingContextStructIndices.Add(Index);
		}
		else
		{
			// Parse the categories and subcategories and construct the data to build the menus.
			TArray<FString> Categories;
			ContextStruct.Category.ParseIntoArray(Categories, TEXT("|"));
			BuildContextStructCategoryRecursive(Categories, Section->SubCategories, Index);
		}
	}

	const bool bDisplayLinkIcon = Args.bUseLinkIconStyle && Args.bCanDisplayLinkIcon;
	TSharedRef<SWidget> LinkIcon = bDisplayLinkIcon ?
		SNew(SBox)
		.HeightOverride(16.0f)
		[
			SNew(SImage)
			.Image(this, &SPropertyBinding::GetLinkIcon)
		]
		:
		SNullWidget::NullWidget;

	//Adjust padding to avoid non centered content and wasted space.
	const float TextLeftPadding = !bDisplayLinkIcon || Args.bLinkIconStyleCompactMode ? 0.0f : 4.0f;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SComboButton)
			.ButtonStyle(Args.BindButtonStyle)
			.ToolTipText(this, &SPropertyBinding::GetCurrentBindingToolTipText)
			.OnGetMenuContent(this, &SPropertyBinding::OnGenerateDelegateMenu)
			.ContentPadding(1)
			.HasDownArrow(!Args.bUseLinkIconStyle)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					LinkIcon
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.HeightOverride(16.0f)
					[
						SNew(SImage)
						.Image(this, &SPropertyBinding::GetCurrentBindingImage)
						.ColorAndOpacity(this, &SPropertyBinding::GetCurrentBindingColor)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(TextLeftPadding, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(this, &SPropertyBinding::GetCurrentBindingText)
					.ColorAndOpacity(this, &SPropertyBinding::GetCurrentBindingTextColor)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.Visibility(this, &SPropertyBinding::GetGotoBindingVisibility)
			.OnClicked(this, &SPropertyBinding::HandleGotoBindingClicked)
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("GotoFunction", "Goto Function"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Search"))
			]
		]
	];
}

bool SPropertyBinding::IsClassDenied(const FPropertyBindingWidgetArgs& Args, UClass* InClass)
{
	if(Args.OnCanBindToClass.IsBound() && Args.OnCanBindToClass.Execute(InClass))
	{
		return false;
	}

	return true;
}

bool SPropertyBinding::IsFieldFromDeniedClass(const FPropertyBindingWidgetArgs& Args, FFieldVariant Field)
{
	return IsClassDenied(Args, Field.GetOwnerClass());
}

template <typename Predicate>
void SPropertyBinding::ForEachBindableFunction(const FPropertyBindingWidgetArgs& Args, UClass* FromClass, Predicate Pred)
{
	if(Args.OnCanBindFunction.IsBound())
	{
		auto CheckBindableClass = [&Args, &Pred](UClass* InBindableClass)
		{
			// Early exit if this class is excluded by permissions
			if ( !FBlueprintActionDatabase::IsClassAllowed(InBindableClass, FBlueprintActionDatabase::EPermissionsContext::Property) )
			{
				return;
			}
		
			// Walk up class hierarchy for native functions and properties
			for ( TFieldIterator<UFunction> FuncIt(InBindableClass, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); FuncIt; ++FuncIt )
			{
				UFunction* Function = *FuncIt;

				// Stop processing functions after reaching a base class that it doesn't make sense to go beyond.
				if ( IsFieldFromDeniedClass(Args, Function) )
				{
					break;
				}

				// Only allow binding pure functions if we're limited to pure function bindings.
				if ( Args.bGeneratePureBindings && !Function->HasAnyFunctionFlags(FUNC_Const | FUNC_BlueprintPure) )
				{
					continue;
				}

				// Only bind to functions that are callable from blueprints
				if ( !UEdGraphSchema_K2::CanUserKismetCallFunction(Function) )
				{
					continue;
				}

				// Only bind to functions that are allowed by permissions
				if ( !FBlueprintActionDatabase::IsFunctionAllowed(Function, FBlueprintActionDatabase::EPermissionsContext::Property) )
				{
					continue;
				}

				bool bValidObjectFunction = false;
				if(Args.bAllowUObjectFunctions)
				{
					FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(Function->GetReturnProperty());
					if(ObjectPropertyBase != nullptr && Function->NumParms == 1)
					{
						bValidObjectFunction = true;
					}
				}

				bool bValidStructFunction = false;
				if(Args.bAllowStructFunctions)
				{
					FStructProperty* StructProperty = CastField<FStructProperty>(Function->GetReturnProperty());
					if(StructProperty != nullptr && Function->NumParms == 1)
					{
						bValidStructFunction = true;
					}
				}

				bool bValidThreadSafety = true;
				if (Args.bAllowOnlyThreadSafeFunctions)
				{
					bValidThreadSafety = FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(Function);
				}

				if(bValidThreadSafety && (bValidObjectFunction || bValidStructFunction || Args.OnCanBindFunction.Execute(Function)))
				{
					Pred(FFunctionInfo(Function));
				}
			}
		};

		CheckBindableClass(FromClass);

		if(Args.bAllowFunctionLibraryBindings)
		{
			// Iterate function libraries
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				UClass* TestClass = *ClassIt;
				if (TestClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass()) && (!TestClass->HasAnyClassFlags(CLASS_Abstract)))
				{
					CheckBindableClass(TestClass);
				}
			}
		}
	}
}

void SPropertyBinding::AddCategoryToMenu(FMenuBuilder& MenuBuilder, const FBindingContextStructCategory& Category)
{
	if (HasCategorySomethingToDisplayRecursive(Category))
	{
		TSharedRef<SWidget> SubMenu =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpacer)
					.Size(FVector2D(18.0f, 0.0f))
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(Category.Name)
				];

		MenuBuilder.AddSubMenu(SubMenu, FNewMenuDelegate::CreateSP(this, &SPropertyBinding::FillCategoryMenu, &Category));
	}
}

void SPropertyBinding::BuildContextStructCategoryRecursive(TConstArrayView<FString> CategoryNames, TArray<FBindingContextStructCategory>& ParentSubCategories, int32 ContextStructIndex)
{
	if (CategoryNames.IsEmpty())
	{
		return;
	}

	FString CategoryName = CategoryNames[0];
	FBindingContextStructCategory* CategoryPtr = ParentSubCategories.FindByPredicate([&](const FBindingContextStructCategory& Category) { return Category.Name.ToString() == CategoryName; });
	if (!CategoryPtr)
	{
		CategoryPtr = &ParentSubCategories.AddDefaulted_GetRef();
		CategoryPtr->Name = FText::FromString(CategoryName);
	}
	
	if(CategoryNames.Num() == 1)
	{
		CategoryPtr->BindingContextStructIndices.Add(ContextStructIndex);
	}
	else
	{
		BuildContextStructCategoryRecursive(CategoryNames.RightChop(1), CategoryPtr->SubCategories, ContextStructIndex);
	}
}

bool SPropertyBinding::HasCategorySomethingToDisplayRecursive(const FBindingContextStructCategory& Category) const
{
	for(const FBindingContextStructCategory& SubCategory : Category.SubCategories)
	{
		if (HasCategorySomethingToDisplayRecursive(SubCategory))
		{
			return true;
		}
	}

	for(const int32 ContextStructIndex : Category.BindingContextStructIndices)
	{
		const FBindingContextStruct& ContextStruct = BindingContextStructs[ContextStructIndex];
		UStruct* Struct = const_cast<UStruct*>(ContextStruct.Struct);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bDeprecatedCanBindToContextStruct = Args.OnCanBindToContextStruct.IsBound() && Args.OnCanBindToContextStruct.Execute(Struct);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (bDeprecatedCanBindToContextStruct || (Args.OnCanBindToContextStructWithIndex.IsBound() && Args.OnCanBindToContextStructWithIndex.Execute(Struct, ContextStructIndex)))
		{
			return true;
		}

		TArray<TSharedPtr<FBindingChainElement>> BindingChain;
		BindingChain.Emplace(MakeShared<FBindingChainElement>(nullptr, ContextStructIndex));
		if (HasBindableProperties(Args, Blueprint, Struct, BindingChain))
		{
			return true;
		}
	}

	return false;
}

TSharedRef<SWidget> SPropertyBinding::MakeContextStructWidget(const FBindingContextStruct& ContextStruct) const
{
	const FText DisplayText = ContextStruct.DisplayText.IsEmpty() ? ContextStruct.Struct->GetDisplayNameText() : ContextStruct.DisplayText;
	const FText ToolTipText = ContextStruct.TooltipText.IsEmpty() ? DisplayText : ContextStruct.TooltipText;

	const FSlateBrush* Icon = ContextStruct.Icon;
	FLinearColor IconColor = FLinearColor::White;
	if (Icon == nullptr)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType PinType;

		if (UClass* Class = Cast<UClass>(ContextStruct.Struct))
		{
			PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
			PinType.PinSubCategory = NAME_None;
			PinType.PinSubCategoryObject = Class;
		}
		else if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ContextStruct.Struct))
		{
			PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinType.PinSubCategory = NAME_None;
			PinType.PinSubCategoryObject = ScriptStruct;
		}
		Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
		IconColor = Schema->GetPinTypeColor(PinType);
	}

	if (ContextStruct.Color.IsSet())
	{
		IconColor = ContextStruct.Color.GetValue();
	}

	return SNew(SHorizontalBox)
			   .ToolTipText(ToolTipText)
		+ SHorizontalBox::Slot()
			  .AutoWidth()
				  [SNew(SSpacer)
						  .Size(FVector2D(18.0f, 0.0f))]
		+ SHorizontalBox::Slot()
			  .AutoWidth()
			  .VAlign(VAlign_Center)
			  .Padding(1.0f, 0.0f)
				  [SNew(SImage)
						  .Image(Icon)
						  .ColorAndOpacity(IconColor)]
		+ SHorizontalBox::Slot()
			  .AutoWidth()
			  .VAlign(VAlign_Center)
			  .Padding(4.0f, 0.0f)
				  [SNew(STextBlock)
						  .Text(DisplayText)];
}

template <typename Predicate>
void SPropertyBinding::ForEachBindableProperty(const FPropertyBindingWidgetArgs& Args,  const UBlueprint* Blueprint, UStruct* InStruct, const TArray<TSharedPtr<FBindingChainElement>>& BindingChain, Predicate Pred)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bIsOnCanBindPropertyBound = Args.OnCanBindProperty.IsBound() || Args.OnCanBindPropertyWithBindingChain.IsBound();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if(InStruct && bIsOnCanBindPropertyBound)
	{
		UBlueprintGeneratedClass* SkeletonClass = Blueprint ? Cast<UBlueprintGeneratedClass>(Blueprint->SkeletonGeneratedClass) : nullptr;

		for ( TFieldIterator<FProperty> PropIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt )
		{
			FProperty* Property = *PropIt;

			// Stop processing properties after reaching the stopped base class
			if ( IsFieldFromDeniedClass(Args, Property) )
			{
				break;
			}

			if (!CanAcceptPropertyOrChildren(Args, Property, BindingChain))
			{
				continue;
			}
			if (SkeletonClass)
			{
				if (!UEdGraphSchema_K2::CanUserKismetAccessVariable(Property, SkeletonClass, UEdGraphSchema_K2::CannotBeDelegate))
				{
					continue;
				}
			}

			// Also ignore advanced properties
			if (Property->HasAnyPropertyFlags(CPF_AdvancedDisplay | CPF_EditorOnly))
			{
				continue;
			}

			// Exit if this property is excluded by permissions
			if (!GetMutableDefault<UPersonaOptions>()->IsAllowedProperty(Property))
			{
				continue;
			}

			Pred(Property);
		}
	}
}

bool SPropertyBinding::HasBindableProperties(const FPropertyBindingWidgetArgs& Args, const UBlueprint* Blueprint, UStruct* InStruct, TArray<TSharedPtr<FBindingChainElement>>& BindingChain)
{
	TSet<UStruct*> VisitedStructs;
	return HasBindablePropertiesRecursive(Args, Blueprint, InStruct, VisitedStructs, BindingChain);
}

bool SPropertyBinding::HasBindablePropertiesRecursive(const FPropertyBindingWidgetArgs& Args, const UBlueprint* Blueprint, UStruct* InStruct, TSet<UStruct*>& VisitedStructs, TArray<TSharedPtr<FBindingChainElement>>& BindingChain)
{
	if (VisitedStructs.Contains(InStruct))
	{
		// If we have visited this struct already and it had bindable properties, it has been accounted for already.
		// Returning false to allow early exit.
		return false;
	}
	VisitedStructs.Add(InStruct);
	
	// Arbitrary cut off to avoid infinite loops.
	if (BindingChain.Num() > Args.MaxDepth)
	{
		return false;
	}
	
	int32 BindableCount = 0;
	ForEachBindableProperty(Args, Blueprint, InStruct, BindingChain, [&Args, Blueprint, &BindableCount, &VisitedStructs, &BindingChain] (FProperty* Property)
	{
		BindingChain.Emplace(MakeShared<FBindingChainElement>(Property));
		ON_SCOPE_EXIT{ BindingChain.Pop(); };

		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);

		if(CanBindProperty(Args, Property, BindingChain))
		{
			BindableCount++;
			return;
		}

		if(Args.bAllowArrayElementBindings && ArrayProperty != nullptr)
		{
			BindingChain.Emplace(MakeShared<FBindingChainElement>(Property));
			BindingChain.Last()->ArrayIndex = 0;
			ON_SCOPE_EXIT{ BindingChain.Pop(); };

			if(CanBindProperty(Args, ArrayProperty->Inner, BindingChain))
			{
				BindableCount++;
				return;
			}	
		}

		FProperty* InnerProperty = Property;
		if(Args.bAllowArrayElementBindings && ArrayProperty != nullptr)
		{
			InnerProperty = ArrayProperty->Inner;
		}

		FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InnerProperty);
		FStructProperty* StructProperty = CastField<FStructProperty>(InnerProperty);

		UStruct* Struct = nullptr;
		UClass* Class = nullptr;

		if ( ObjectProperty )
		{
			Struct = Class = ObjectProperty->PropertyClass;
		}
		else if ( StructProperty )
		{
			Struct = StructProperty->Struct;
		}

		if (Struct)
		{
			if (UStruct* ResolvedStruct = ResolveIndirection(Args, BindingChain))
			{
				Struct = ResolvedStruct;
			}

			// Recurse into a struct, except if it is the same type as the one we're binding.
			if (HasBindablePropertiesRecursive(Args, Blueprint, Struct, VisitedStructs, BindingChain))
			{
				if (Class)
				{
					// Ignore any subobject properties that are not bindable.
					// Also ignore any class that is explicitly on the deny list.
					if (IsClassDenied(Args, Class) || (Args.OnCanBindToSubObjectClass.IsBound() && Args.OnCanBindToSubObjectClass.Execute(Class)))
					{
						return;
					}
				}

				if (Args.bAllowArrayElementBindings && ArrayProperty != nullptr)
				{
					BindableCount++;
				}
				else if (Args.bAllowStructMemberBindings)
				{
					BindableCount++;
				}
			}
		}
	});

	if(UClass* Class = Cast<UClass>(InStruct))
	{
		ForEachBindableFunction(Args, Class, [&Args, &Blueprint, &BindableCount, &VisitedStructs, &BindingChain](const FFunctionInfo& Info)
		{
			BindingChain.Emplace(MakeShared<FBindingChainElement>(Info.Function));
			ON_SCOPE_EXIT{ BindingChain.Pop(); };

			FProperty* ReturnProperty = Info.Function->GetReturnProperty();
			
			// We can get here if we accept non-leaf UObject functions, so if so we need to check the return value for compatibility
			if(!Args.bAllowUObjectFunctions || CanBindProperty(Args, ReturnProperty, BindingChain))
			{
				BindableCount++;
			}

			// Only show bindable subobjects, structs and variables if we're generating pure bindings.
			if(Args.bGeneratePureBindings)
			{
				if(FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(ReturnProperty))
				{
					if (HasBindablePropertiesRecursive(Args, Blueprint,ObjectPropertyBase->PropertyClass, VisitedStructs, BindingChain))
					{
						if (Args.bAllowUObjectFunctions)
						{
							BindableCount++;
						}
					}
				}
				else if(FStructProperty* StructProperty = CastField<FStructProperty>(ReturnProperty))
				{
					if (HasBindablePropertiesRecursive(Args, Blueprint, StructProperty->Struct, VisitedStructs, BindingChain))
					{
						BindableCount++;
					}
				}
			}
		});
	}
	
	return BindableCount > 0;
}

TSharedRef<SWidget> SPropertyBinding::OnGenerateDelegateMenu()
{
	const bool bShouldExecuteCustomGenerateMenu = Args.OnGenerateMenu.IsBound() && (!Args.OnCanUseCustomGenerateMenu.IsBound() || Args.OnCanUseCustomGenerateMenu.Execute());
	if (bShouldExecuteCustomGenerateMenu)
	{
		return Args.OnGenerateMenu.Execute();
	}

	// The menu itself is be searchable.
	const bool bSearchableMenu = true;

	// The menu are generated through reflection and sometime the API exposes some recursivity (think about a Widget returning its parent which is also a Widget). Just by reflection
	// it is not possible to determine when the root object is reached. It needs a kind of simulation which is not implemented. Also, even if the recursivity was correctly handled, the possible
	// permutations tend to grow exponentially. Until a clever solution is found, the simple approach is to disable recursively searching those menus. User can still search the current one though.
	const bool bRecursivelySearchableMenu = false;

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr, Args.MenuExtender, /*bCloseSelfOnly*/false, &FCoreStyle::Get(), bSearchableMenu, NAME_None, bRecursivelySearchableMenu);

	MenuBuilder.BeginSection("BindingActions", LOCTEXT("Bindings", "Bindings"));

	if ( CanRemoveBinding() )
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveBinding", "Remove Binding"),
			LOCTEXT("RemoveBindingToolTip", "Removes the current binding"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cross"),
			FUIAction(FExecuteAction::CreateSP(this, &SPropertyBinding::HandleRemoveBinding))
			);
	}

	if(Args.bAllowNewBindings && Args.BindableSignature != nullptr)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateBinding", "Create Binding"),
			LOCTEXT("CreateBindingToolTip", "Creates a new function on the widget blueprint that will return the binding data for this property."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Plus"),
			FUIAction(FExecuteAction::CreateSP(this, &SPropertyBinding::HandleCreateAndAddBinding))
			);
	}

	MenuBuilder.EndSection(); //CreateBinding

	// Properties
	{
		// Get the current skeleton class, think header for the blueprint.
		UBlueprintGeneratedClass* SkeletonClass = Blueprint ? Cast<UBlueprintGeneratedClass>(Blueprint->SkeletonGeneratedClass) : nullptr;

		TArray<TSharedPtr<FBindingChainElement>> BindingChain;
		FillPropertyMenu(MenuBuilder, SkeletonClass, BindingChain);
	}

	if (BindingContextStructs.Num() > 0)
	{
		// Go thought each top level section and display the sub categories as menu and then the context struct without categories.
		for(const FBindingContextStructCategory& Section : BindingContextStructSections)
		{
			MenuBuilder.BeginSection(FName(), Section.Name);
			for(const FBindingContextStructCategory& SubCategory : Section.SubCategories)
			{
				AddCategoryToMenu(MenuBuilder, SubCategory);
			}

			for (int32 Index : Section.BindingContextStructIndices)
			{
				FBindingContextStruct& ContextStruct = BindingContextStructs[Index];

				// Make first chain element representing the index in the context array.
				TArray<TSharedPtr<FBindingChainElement>> BindingChain;
				BindingChain.Emplace(MakeShared<FBindingChainElement>(nullptr, Index));

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				const bool bDeprecatedCanBindToContextStruct = Args.OnCanBindToContextStruct.IsBound() && Args.OnCanBindToContextStruct.Execute(ContextStruct.Struct);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (bDeprecatedCanBindToContextStruct || (Args.OnCanBindToContextStructWithIndex.IsBound() && Args.OnCanBindToContextStructWithIndex.Execute(ContextStruct.Struct, Index)))
				{
					// If the struct can be be bound to directly, create action for that.
					MenuBuilder.AddMenuEntry(
						FUIAction(FExecuteAction::CreateSP(this, &SPropertyBinding::HandleAddBinding, BindingChain)),
						MakeContextStructWidget(ContextStruct));
				}
				if (HasBindableProperties(Args, Blueprint, ContextStruct.Struct, BindingChain))
				{
					// Show struct properties.
					MenuBuilder.AddSubMenu(
						MakeContextStructWidget(ContextStruct),
						FNewMenuDelegate::CreateSP(this, &SPropertyBinding::FillPropertyMenu, ContextStruct.Struct, BindingChain));
				}
			}
			MenuBuilder.EndSection();
		}
	}

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);

	return
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.MaxHeight(DisplayMetrics.PrimaryDisplayHeight * 0.5)
		[
			MenuBuilder.MakeWidget()
		];
}

void SPropertyBinding::FillPropertyMenu(FMenuBuilder& MenuBuilder, UStruct* InOwnerStruct, TArray<TSharedPtr<FBindingChainElement>> InBindingChain)
{
	FillPropertyMenu(MenuBuilder, Args, Blueprint, PropertyName, InOwnerStruct, InBindingChain);	
}

void SPropertyBinding::FillPropertyMenu(FMenuBuilder& MenuBuilder, const FPropertyBindingWidgetArgs& Args, const UBlueprint* Blueprint, FName PropertyName,  UStruct* InOwnerStruct, TArray<TSharedPtr<FBindingChainElement>> InBindingChain)
{
	auto MakeArrayElementPropertyWidget = [&Args, PropertyName](FProperty* InProperty, const TArray<TSharedPtr<FBindingChainElement>>& InNewBindingChain)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType PinType;
		Schema->ConvertPropertyToPinType(InProperty, PinType);
		
		TSharedPtr<FBindingChainElement> LeafElement = InNewBindingChain.Last();

		return SNew(SHorizontalBox)
			.ToolTipText(InProperty->GetToolTipText())
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpacer)
				.Size(FVector2D(18.0f, 0.0f))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FBlueprintEditorUtils::GetIconFromPin(PinType, true))
				.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromName(InProperty->GetFName()))
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.Padding(2.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(24.0f)
				[
					SNew(SNumericEntryBox<int32>)
					.ToolTipText(LOCTEXT("ArrayIndex", "Array Index"))
					.Value_Lambda([LeafElement](){ return LeafElement->ArrayIndex == INDEX_NONE ? TOptional<int32>() : TOptional<int32>(LeafElement->ArrayIndex); })
					.OnValueCommitted_Lambda([Args, PropertyName, InProperty, InNewBindingChain](int32 ArrayIndex, ETextCommit::Type TextCommitType)
						{
							HandleSetBindingArrayIndex(Args, PropertyName, ArrayIndex, TextCommitType, InProperty, InNewBindingChain);
						})
				]
			];
	};

	auto MakeArrayElementEntry = [&Args, PropertyName, &MenuBuilder, &MakeArrayElementPropertyWidget](FProperty* InProperty, const TArray<TSharedPtr<FBindingChainElement>>& InNewBindingChain)
	{
		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction::CreateLambda([Args, PropertyName, InNewBindingChain]()
				{
					HandleAddBinding(Args, PropertyName, InNewBindingChain);
				})),
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				MakeArrayElementPropertyWidget(InProperty, InNewBindingChain)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpacer)
				.Size(FVector2D(15.0f, 0.0f))
			]);
	};

	auto MakePropertyWidget = [](FProperty* InProperty)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType PinType;
		Schema->ConvertPropertyToPinType(InProperty, PinType);

		FText PropertyDisplayName;
		if(InProperty->IsNative())
		{
			if(const FString* ScriptNamePtr = InProperty->FindMetaData("ScriptName"))
			{
				PropertyDisplayName = FText::FromString(*ScriptNamePtr);
			}
			else
			{
				PropertyDisplayName = FText::FromString(InProperty->GetName());
			}
		}
		else
		{
			PropertyDisplayName = FText::FromString(InProperty->GetDisplayNameText().ToString());
		}
		
		return SNew(SHorizontalBox)
			.ToolTipText(InProperty->GetToolTipText())
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpacer)
				.Size(FVector2D(18.0f, 0.0f))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FBlueprintEditorUtils::GetIconFromPin(PinType, true))
				.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(PropertyDisplayName)
			];
	};

	auto MakePropertyEntry = [&Args, PropertyName, &MenuBuilder, &MakePropertyWidget](FProperty* InProperty, const TArray<TSharedPtr<FBindingChainElement>>& InNewBindingChain)
	{
		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction::CreateLambda(
				[Args, PropertyName, InNewBindingChain]()
				{
					HandleAddBinding(Args, PropertyName, InNewBindingChain);
				})),
			MakePropertyWidget(InProperty));
	};

	auto MakeFunctionWidget = [](const FFunctionInfo& Info)
	{
		static FName FunctionIcon(TEXT("GraphEditor.Function_16x"));

		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType PinType;
		if(FProperty* ReturnProperty = Info.Function->GetReturnProperty())
		{
			Schema->ConvertPropertyToPinType(ReturnProperty, PinType);
		}

		return SNew(SHorizontalBox)
			.ToolTipText(FText::FromString(Info.Tooltip))
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpacer)
				.Size(FVector2D(18.0f, 0.0f))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush(FunctionIcon))
				.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(Info.DisplayName)
			];
	};

	//---------------------------------------
	// Function Bindings

	if (Args.bAllowFunctionBindings)
	{
		if ( UClass* OwnerClass = Cast<UClass>(InOwnerStruct) )
		{
			static FName FunctionIcon(TEXT("GraphEditor.Function_16x"));

			MenuBuilder.BeginSection("Functions", LOCTEXT("Functions", "Functions"));
			{
				ForEachBindableFunction(Args, OwnerClass, [&Args, Blueprint, PropertyName, &InBindingChain, &MenuBuilder, &MakeFunctionWidget] (const FFunctionInfo& Info) 
				{
					TArray<TSharedPtr<FBindingChainElement>> NewBindingChain(InBindingChain);
					NewBindingChain.Emplace(MakeShared<FBindingChainElement>(Info.Function));

					FProperty* ReturnProperty = Info.Function->GetReturnProperty();
					// We can get here if we accept non-leaf UObject functions, so if so we need to check the return value for compatibility
					if(!Args.bAllowUObjectFunctions || CanBindProperty(Args, ReturnProperty, InBindingChain))
					{
						MenuBuilder.AddMenuEntry(
							FUIAction(FExecuteAction::CreateLambda(
								[Args, PropertyName, NewBindingChain]()
									{
										HandleAddBinding(Args, PropertyName, NewBindingChain);
									})),
							MakeFunctionWidget(Info));
					}

					// Only show bindable subobjects, structs and variables if we're generating pure bindings.
					if(Args.bGeneratePureBindings)
					{
						if(FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(ReturnProperty))
						{
							if (HasBindableProperties(Args, Blueprint, ObjectPropertyBase->PropertyClass,  NewBindingChain))
							{
								MenuBuilder.AddSubMenu(
									MakeFunctionWidget(Info),
									FNewMenuDelegate::CreateLambda(
										[Args, Blueprint, PropertyName, ObjectPropertyBase, NewBindingChain](FMenuBuilder& MenuBuilder)
											{
												FillPropertyMenu(MenuBuilder, Args, Blueprint, PropertyName, static_cast<UStruct*>(ObjectPropertyBase->PropertyClass), NewBindingChain);
											}));
							}
						}
						else if(FStructProperty* StructProperty = CastField<FStructProperty>(ReturnProperty))
						{
							if (HasBindableProperties(Args, Blueprint, StructProperty->Struct, NewBindingChain))
							{
								MenuBuilder.AddSubMenu(
									MakeFunctionWidget(Info),
									FNewMenuDelegate::CreateLambda([Args, Blueprint, PropertyName, StructProperty, NewBindingChain](FMenuBuilder& MenuBuilder)
								{
									FillPropertyMenu(MenuBuilder, Args, Blueprint, PropertyName, static_cast<UStruct*>(StructProperty->Struct), NewBindingChain);
								}));
							}
						}
					}
				});
			}
			MenuBuilder.EndSection(); //Functions
		}
	}

	//---------------------------------------
	// Property Bindings

	// Only show bindable subobjects and variables if we're generating pure bindings.
	if (Args.bGeneratePureBindings && Args.bAllowPropertyBindings)
	{
		FProperty* BindingProperty = nullptr;
		if (Args.BindableSignature != nullptr)
		{
			BindingProperty = Args.BindableSignature->GetReturnProperty();
		}
		else
		{
			BindingProperty = Args.Property;
		}

		// Find the binder that can handle the delegate return type, don't bother allowing people 
		// to look for bindings that we don't support
		if (CanBindProperty(Args, BindingProperty, InBindingChain))
		{
			UStruct* BindingStruct = nullptr;
			FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(BindingProperty);
			FStructProperty* StructProperty = CastField<FStructProperty>(BindingProperty);
			if (ObjectProperty)
			{
				BindingStruct = ObjectProperty->PropertyClass;
			}
			else if (StructProperty)
			{
				BindingStruct = StructProperty->Struct;
			}

			MenuBuilder.BeginSection("Properties", LOCTEXT("Properties", "Properties"));
			{
				ForEachBindableProperty(Args, Blueprint, InOwnerStruct, InBindingChain, [&Args, Blueprint, PropertyName, &InBindingChain, BindingStruct, &MenuBuilder, &MakeArrayElementPropertyWidget, &MakePropertyWidget, &MakePropertyEntry, &MakeArrayElementEntry](FProperty* Property)
					{
						TArray<TSharedPtr<FBindingChainElement>> NewBindingChain(InBindingChain);
						TArray<TSharedPtr<FBindingChainElement>> NewArrayElementBindingChain;
						NewBindingChain.Emplace(MakeShared<FBindingChainElement>(Property));

						if (CanBindProperty(Args, Property, NewBindingChain))
						{
							MakePropertyEntry(Property, NewBindingChain);
						}

						FProperty* InnerProperty = Property;
						FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);

						const bool bUseArrayProperty = Args.bAllowArrayElementBindings && ArrayProperty != nullptr;
						if (bUseArrayProperty)
						{
							NewArrayElementBindingChain = InBindingChain;
							NewArrayElementBindingChain.Emplace(MakeShared<FBindingChainElement>(Property));
							NewArrayElementBindingChain.Last()->ArrayIndex = 0;
							if (CanBindProperty(Args, ArrayProperty->Inner, NewArrayElementBindingChain))
							{
								MakeArrayElementEntry(ArrayProperty->Inner, NewArrayElementBindingChain);
							}

							InnerProperty = ArrayProperty->Inner;
						}

						FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InnerProperty);
						FStructProperty* StructProperty = CastField<FStructProperty>(InnerProperty);

						UStruct* Struct = nullptr;
						UClass* Class = nullptr;

						if (ObjectProperty)
						{
							Struct = Class = ObjectProperty->PropertyClass;
						}
						else if (StructProperty)
						{
							Struct = StructProperty->Struct;
						}

						// Recurse into a struct if it has some properties we can bind to.
						if (Struct)
						{
							if (UStruct* ResolvedStruct = ResolveIndirection(Args, NewBindingChain))
							{
								Struct = ResolvedStruct;
							}

							if (Class)
							{
								// Ignore any subobject properties that are not bindable.
								// Also ignore any class that is explicitly on the deny list.
								if (IsClassDenied(Args, Class) || (Args.OnCanBindToSubObjectClass.IsBound() && Args.OnCanBindToSubObjectClass.Execute(Class)))
								{
									return;
								}
							}

							TArray<TSharedPtr<FBindingChainElement>>& BindablePropertyBindingChain = bUseArrayProperty
								? NewArrayElementBindingChain
								: NewBindingChain;
							if (HasBindableProperties(Args, Blueprint, Struct, BindablePropertyBindingChain))
							{
								if (bUseArrayProperty)
								{
									MenuBuilder.AddSubMenu(
										MakeArrayElementPropertyWidget(ArrayProperty->Inner, NewArrayElementBindingChain),
										FNewMenuDelegate::CreateLambda(
											[Args, Blueprint, PropertyName, Struct, NewArrayElementBindingChain](FMenuBuilder& MenuBuilder)
												{
													FillPropertyMenu(MenuBuilder, Args, Blueprint, PropertyName, Struct, NewArrayElementBindingChain);
												})
									);
								}
								else if (Args.bAllowStructMemberBindings)
								{
									MenuBuilder.AddSubMenu(
										MakePropertyWidget(Property),
										FNewMenuDelegate::CreateLambda(
											[Args, Blueprint, PropertyName, Struct, NewBindingChain](FMenuBuilder& MenuBuilder)
												{
													FillPropertyMenu(MenuBuilder, Args, Blueprint, PropertyName, Struct, NewBindingChain);
												})
									);
								}
							}
						}
					});
			}
			MenuBuilder.EndSection(); //Properties
		}
	}

	// Add 'none' entry only if we just have the search block in the builder
	if ( InOwnerStruct && MenuBuilder.GetMultiBox()->GetBlocks().Num() == 1 )
	{
		MenuBuilder.BeginSection("None", InOwnerStruct->GetDisplayNameText());
		MenuBuilder.AddWidget(SNew(STextBlock).Text(LOCTEXT("None", "None")), FText::GetEmpty());
		MenuBuilder.EndSection(); //None
	}
}

void SPropertyBinding::FillCategoryMenu(FMenuBuilder& MenuBuilder, const FBindingContextStructCategory* Category)
{
	MenuBuilder.BeginSection(NAME_None, Category->Name);
	for (const FBindingContextStructCategory& SubCategory : Category->SubCategories)
	{
		AddCategoryToMenu(MenuBuilder, SubCategory);
	}

	for(int32 Index : Category->BindingContextStructIndices)
	{
		FBindingContextStruct& ContextStruct = BindingContextStructs[Index];

		// Make first chain element representing the index in the context array.
		TArray<TSharedPtr<FBindingChainElement>> BindingChain;
		BindingChain.Emplace(MakeShared<FBindingChainElement>(nullptr, Index));

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bDeprecatedCanBindToContextStruct = Args.OnCanBindToContextStruct.IsBound() && Args.OnCanBindToContextStruct.Execute(ContextStruct.Struct);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (bDeprecatedCanBindToContextStruct || (Args.OnCanBindToContextStructWithIndex.IsBound() && Args.OnCanBindToContextStructWithIndex.Execute(ContextStruct.Struct, Index)))
		{
			// If the struct can be be bound to directly, create action for that.
			MenuBuilder.AddMenuEntry(
				FUIAction(FExecuteAction::CreateSP(this, &SPropertyBinding::HandleAddBinding, BindingChain)),
				MakeContextStructWidget(ContextStruct));
		}
		if (HasBindableProperties(Args, Blueprint, ContextStruct.Struct, BindingChain))
		{
			// Show struct properties.
			MenuBuilder.AddSubMenu(
				MakeContextStructWidget(ContextStruct),
				FNewMenuDelegate::CreateSP(this, &SPropertyBinding::FillPropertyMenu, ContextStruct.Struct, BindingChain));
		}
	}
	MenuBuilder.EndSection();
}

const FSlateBrush* SPropertyBinding::GetLinkIcon() const
{
	bool CheckState = HasAnyBindings();
	return (CheckState) ? FAppStyle::GetBrush("Icons.Link") : FAppStyle::GetBrush("Icons.Unlink");
}

const FSlateBrush* SPropertyBinding::GetCurrentBindingImage() const
{
	if (Args.bLinkIconStyleCompactMode)
	{
		return nullptr;
	}

	if(Args.CurrentBindingImage.IsSet())
	{
		return Args.CurrentBindingImage.Get();
	}

	return nullptr;
}

FText SPropertyBinding::GetCurrentBindingText() const
{
	if (Args.bLinkIconStyleCompactMode)
	{
		return FText::GetEmpty();
	}

	if(Args.CurrentBindingText.IsSet())
	{
		return Args.CurrentBindingText.Get();
	}

	return LOCTEXT("Bind", "Bind");
}

FSlateColor SPropertyBinding::GetCurrentBindingTextColor() const
{
	if (Args.CurrentBindingTextColor.IsSet())
	{
		return Args.CurrentBindingTextColor.Get();
	}

	return FSlateColor::UseForeground();
}

FText SPropertyBinding::GetCurrentBindingToolTipText() const
{
	if(Args.CurrentBindingToolTipText.IsSet())
	{
		return Args.CurrentBindingToolTipText.Get();
	}

	return GetCurrentBindingText();
}

FSlateColor SPropertyBinding::GetCurrentBindingColor() const
{
	if(Args.CurrentBindingColor.IsSet())
	{
		return Args.CurrentBindingColor.Get();
	}

	return FLinearColor(0.25f, 0.25f, 0.25f);
}

bool SPropertyBinding::CanRemoveBinding()
{
	if(Args.OnCanRemoveBinding.IsBound())
	{
		return Args.OnCanRemoveBinding.Execute(PropertyName);
	}

	return false;
}

void SPropertyBinding::HandleRemoveBinding()
{
	if(Args.OnRemoveBinding.IsBound())
	{
		const FScopedTransaction Transaction(LOCTEXT("UnbindDelegate", "Remove Binding"));

		Args.OnRemoveBinding.Execute(PropertyName);
	}
}

void SPropertyBinding::HandleAddBinding(const TArray<TSharedPtr<FBindingChainElement>> InBindingChain) const
{
	HandleAddBinding(Args, PropertyName, InBindingChain);
}

void SPropertyBinding::HandleAddBinding(const FPropertyBindingWidgetArgs& Args, FName PropertyName, const TArray<TSharedPtr<FBindingChainElement>>& InBindingChain)
{
	if(Args.OnAddBinding.IsBound())
	{
		const FScopedTransaction Transaction(LOCTEXT("BindDelegate", "Set Binding"));

		TArray<FBindingChainElement> BindingChain;
		BindingChain.Reserve(InBindingChain.Num());
		Algo::Transform(InBindingChain, BindingChain, [](TSharedPtr<FBindingChainElement> InElement)
		{
			return *InElement.Get();
		});
		Args.OnAddBinding.Execute(PropertyName, BindingChain);
	}
}

void SPropertyBinding::HandleSetBindingArrayIndex(const FPropertyBindingWidgetArgs& Args, FName PropertyName, int32 InArrayIndex, ETextCommit::Type InCommitType, FProperty* InProperty, TArray<TSharedPtr<FBindingChainElement>> InBindingChain)
{
	InBindingChain.Last()->ArrayIndex = InArrayIndex;

	// If the user hit enter on a compatible property, assume they want to accept
	if(CanBindProperty(Args, InProperty, InBindingChain) && InCommitType == ETextCommit::OnEnter)
	{
		HandleAddBinding(Args, PropertyName, InBindingChain);
	}
}

void SPropertyBinding::HandleCreateAndAddBinding()
{
	if (!Blueprint)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateDelegate", "Create Binding"));

	Blueprint->Modify();

	FString Pre = Args.bGeneratePureBindings ? FString(TEXT("Get")) : FString(TEXT("On"));

	FString WidgetName;
	if(Args.OnGenerateBindingName.IsBound())
	{
		WidgetName = Args.OnGenerateBindingName.Execute();
	}

	FString Post = (PropertyName != NAME_None) ? PropertyName.ToString() : FString();
	Post.RemoveFromStart(TEXT("On"));
	Post.RemoveFromEnd(TEXT("Event"));

	// Create the function graph.
	FString FunctionName = Pre + WidgetName + Post;
	UEdGraph* FunctionGraph = FBlueprintEditorUtils::CreateNewGraph(
		Blueprint, 
		FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, FunctionName),
		UEdGraph::StaticClass(),
		UEdGraphSchema_K2::StaticClass());
	
	const bool bUserCreated = true;
	FBlueprintEditorUtils::AddFunctionGraph(Blueprint, FunctionGraph, bUserCreated, Args.BindableSignature);

	UFunction* Function = Blueprint->SkeletonGeneratedClass->FindFunctionByName(FunctionGraph->GetFName());
	check(Function);

	Args.OnNewFunctionBindingCreated.ExecuteIfBound(FunctionGraph, Function);
	
	// Add the binding to the blueprint
	TArray<TSharedPtr<FBindingChainElement>> BindingPath;
	BindingPath.Emplace(MakeShared<FBindingChainElement>(Function));

	HandleAddBinding(Args, PropertyName, BindingPath);

	// Only mark bindings as pure that need to be.
	if ( Args.bGeneratePureBindings )
	{
		const UEdGraphSchema_K2* Schema_K2 = Cast<UEdGraphSchema_K2>(FunctionGraph->GetSchema());
		Schema_K2->AddExtraFunctionFlags(FunctionGraph, FUNC_BlueprintPure);
	}

	HandleGotoBindingClicked();
}

UStruct* SPropertyBinding::ResolveIndirection(const FPropertyBindingWidgetArgs& Args, const TArray<TSharedPtr<FBindingChainElement>>& InBindingChain)
{
	UStruct* ResolvedStruct = nullptr;

	if (Args.OnResolveIndirection.IsBound())
	{
		TArray<FBindingChainElement> BindingChain;
		BindingChain.Reserve(InBindingChain.Num());
		Algo::Transform(InBindingChain, BindingChain, [](TSharedPtr<FBindingChainElement> InElement)
		{
			return *InElement.Get();
		});

		ResolvedStruct = Args.OnResolveIndirection.Execute(BindingChain);
	}

	return ResolvedStruct;
}

EVisibility SPropertyBinding::GetGotoBindingVisibility() const
{
	if(Args.OnCanGotoBinding.IsBound())
	{
		if(Args.OnCanGotoBinding.Execute(PropertyName))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FReply SPropertyBinding::HandleGotoBindingClicked()
{
	if(Args.OnGotoBinding.IsBound())
	{
		if(Args.OnGotoBinding.Execute(PropertyName))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SPropertyBinding::CanAcceptPropertyOrChildren(const FPropertyBindingWidgetArgs& Args, FProperty* InProperty, TConstArrayView<TSharedPtr<FBindingChainElement>> InBindingChain)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Args.OnCanAcceptPropertyOrChildren.IsBound() && Args.OnCanAcceptPropertyOrChildren.Execute(InProperty) == false)
	{
		return false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
			
	if (Args.OnCanAcceptPropertyOrChildrenWithBindingChain.IsBound())
	{
		TArray<FBindingChainElement, TInlineAllocator<32>> BindingChain;
		Algo::Transform(InBindingChain, BindingChain, [](TSharedPtr<FBindingChainElement> InElement)
		{
			return *InElement.Get();
		});

		if (!Args.OnCanAcceptPropertyOrChildrenWithBindingChain.Execute(InProperty, BindingChain))
		{
			return false;
		}
	}

	return true;
}

bool SPropertyBinding::CanBindProperty(const FPropertyBindingWidgetArgs& Args, FProperty* InProperty, TConstArrayView<TSharedPtr<FBindingChainElement>> InBindingChain)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Args.OnCanBindProperty.IsBound() && Args.OnCanBindProperty.Execute(InProperty) == false)
	{
		return false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
			
	if (Args.OnCanBindPropertyWithBindingChain.IsBound())
	{
		TArray<FBindingChainElement, TInlineAllocator<32>> BindingChain;
		Algo::Transform(InBindingChain, BindingChain, [](TSharedPtr<FBindingChainElement> InElement)
		{
			return *InElement.Get();
		});

		if (!Args.OnCanBindPropertyWithBindingChain.Execute(InProperty, BindingChain))
		{
			return false;
		}
	}

	return true;
}

FReply SPropertyBinding::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (Args.OnDrop.IsBound())
	{
		return Args.OnDrop.Execute(MyGeometry, DragDropEvent);
	}

	return FReply::Unhandled();
}

bool SPropertyBinding::HasAnyBindings() const
{
	if (Args.OnHasAnyBindings.IsBound())
	{
		return Args.OnHasAnyBindings.Execute();
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
