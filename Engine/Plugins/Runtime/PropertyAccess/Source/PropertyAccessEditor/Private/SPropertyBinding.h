// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "IPropertyAccessEditor.h"

namespace ETextCommit { enum Type : int; }

class FMenuBuilder;
class UEdGraph;
class UBlueprint;
struct FEditorPropertyPath;

class SPropertyBinding : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPropertyBinding){}

	SLATE_ARGUMENT(FPropertyBindingWidgetArgs, Args)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UBlueprint* InBlueprint, const TArray<FBindingContextStruct>& InBindingContextStructs);

	static void FillPropertyMenu(FMenuBuilder& MenuBuilder,  const FPropertyBindingWidgetArgs& Args, const UBlueprint* Blueprint, FName PropertyName, UStruct* InOwnerStruct, TArray<TSharedPtr<FBindingChainElement>> InBindingChain);
protected:
	struct FFunctionInfo
	{
		FFunctionInfo()
			: Function(nullptr)
		{
		}

		FFunctionInfo(UFunction* InFunction)
			: DisplayName(InFunction->HasMetaData("ScriptName") ? InFunction->GetMetaDataText("ScriptName") : FText::FromName(InFunction->GetFName()))
			, Tooltip(InFunction->GetMetaData("Tooltip"))
			, FuncName(InFunction->GetFName())
			, Function(InFunction)
		{}

		FText DisplayName;
		FString Tooltip;

		FName FuncName;
		UFunction* Function;
	};

	struct FBindingContextStructCategory
	{
		FText Name;
		TArray<FBindingContextStructCategory> SubCategories;
		TArray<int32> BindingContextStructIndices;
	};


	TSharedRef<SWidget> OnGenerateDelegateMenu();
	void FillPropertyMenu(FMenuBuilder& MenuBuilder, UStruct* InOwnerStruct, TArray<TSharedPtr<FBindingChainElement>> InBindingChain);
	void FillCategoryMenu(FMenuBuilder& MenuBuilder, const FBindingContextStructCategory* Category);

	const FSlateBrush* GetLinkIcon() const;
	const FSlateBrush* GetCurrentBindingImage() const;
	FText GetCurrentBindingText() const;
	FSlateColor GetCurrentBindingTextColor() const;
	FText GetCurrentBindingToolTipText() const;
	FSlateColor GetCurrentBindingColor() const;

	bool CanRemoveBinding();
	void HandleRemoveBinding();

	void HandleAddBinding(const TArray<TSharedPtr<FBindingChainElement>> InBindingChain) const;
	static void HandleAddBinding(const FPropertyBindingWidgetArgs& Args, FName PropertyName, const TArray<TSharedPtr<FBindingChainElement>>& InBindingChain);
	static void HandleSetBindingArrayIndex(const FPropertyBindingWidgetArgs& Args, FName PropertyName, int32 InArrayIndex, ETextCommit::Type InCommitType, FProperty* InProperty, TArray<TSharedPtr<FBindingChainElement>> InBindingChain);

	void HandleCreateAndAddBinding();

	static UStruct* ResolveIndirection(const FPropertyBindingWidgetArgs& Args, const TArray<TSharedPtr<FBindingChainElement>>& BindingChain);

	EVisibility GetGotoBindingVisibility() const;

	FReply HandleGotoBindingClicked();

	// Helper function to call the OnCanAcceptProperty* delegates, handles conversion of binding chain to TConstArrayView<FBindingChainElement> as expected by the delegate.
	static bool CanAcceptPropertyOrChildren(const FPropertyBindingWidgetArgs& Args, FProperty* InProperty, TConstArrayView<TSharedPtr<FBindingChainElement>> InBindingChain);
	
	// Helper function to call the OnCanBindProperty* delegates, handles conversion of binding chain to TConstArrayView<FBindingChainElement> as expected by the delegate.
	static bool CanBindProperty(const FPropertyBindingWidgetArgs& Args, FProperty* InProperty, TConstArrayView<TSharedPtr<FBindingChainElement>> InBindingChain);

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	bool HasAnyBindings() const;

private:

	static bool IsClassDenied(const FPropertyBindingWidgetArgs& Args, UClass* OwnerClass);
	static bool IsFieldFromDeniedClass(const FPropertyBindingWidgetArgs& Args, FFieldVariant Field);
	static bool HasBindableProperties(const FPropertyBindingWidgetArgs& Args, const UBlueprint* Blueprint, UStruct* InStruct, TArray<TSharedPtr<FBindingChainElement>>& BindingChain);
	static bool HasBindablePropertiesRecursive(const FPropertyBindingWidgetArgs& Args, const UBlueprint* Blueprint, UStruct* InStruct, TSet<UStruct*>& VisitedStructs, TArray<TSharedPtr<FBindingChainElement>>& BindingChain);

	/**
	 * Note that an ArrayView is not used to pass the BindingChain around since the predicate can modify the array
	 * and this will invalidate the ArrayView if reallocation is performed.
	 */
	template <typename Predicate>
	static void ForEachBindableProperty(const FPropertyBindingWidgetArgs& Args,  const UBlueprint* Blueprint, UStruct* InStruct, const TArray<TSharedPtr<FBindingChainElement>>& BindingChain, Predicate Pred);

	template <typename Predicate>
	static void ForEachBindableFunction(const FPropertyBindingWidgetArgs& Args, UClass* FromClass, Predicate Pred);

	void AddCategoryToMenu(FMenuBuilder& MenuBuilder, const FBindingContextStructCategory& Category);
	void BuildContextStructCategoryRecursive(TConstArrayView<FString> CategoryNames, TArray<FBindingContextStructCategory>& ParentSubCategories, int32 ContextStructIndex);
	bool HasCategorySomethingToDisplayRecursive(const FBindingContextStructCategory& Category) const;

	TSharedRef<SWidget> MakeContextStructWidget(const FBindingContextStruct& ContextStruct) const;

	UBlueprint* Blueprint = nullptr;
	TArray<FBindingContextStruct> BindingContextStructs;
	// Top level sections of the binding ContextStructs
	TArray<FBindingContextStructCategory> BindingContextStructSections;
	FPropertyBindingWidgetArgs Args;
	FName PropertyName;
};
