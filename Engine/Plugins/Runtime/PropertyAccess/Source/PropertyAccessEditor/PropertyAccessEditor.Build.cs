// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PropertyAccessEditor : ModuleRules
{
	public PropertyAccessEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Slate",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"GraphEditor",
				"BlueprintGraph",
				
				"Engine",
				"UnrealEd",
				"SlateCore",
				"InputCore",
				"KismetWidgets",
				"PropertyPath",
				"AnimGraph",
			}
		);
	}
}
